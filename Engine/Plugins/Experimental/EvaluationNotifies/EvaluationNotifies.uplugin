{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Evaluation Notifies",
	"Description": "A system for animation notifies which have animation evaluation time code.",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "EvaluationNotifiesRuntime",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "EvaluationNotifiesEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AnimationWarping",
			"Enabled" : true
		},
		{
			"Name": "RigVM",
			"Enabled" : true
		},
		{
			"Name": "UAF",
			"Enabled" : true
		},
		{
			"Name": "UAFAnimGraph",
			"Enabled" : true
		}
	]
}