// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EvaluationNotifiesEditor : ModuleRules
	{
		public EvaluationNotifiesEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
				"EvaluationNotifiesRuntime",
                "AnimGraph",
				"AnimGraphRuntime",
				"AnimationModifiers",
				"AnimationBlueprintLibrary",
				"Core",
                "CoreUObject",
                "Engine",
            });

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
                "SlateCore",
            });

            if (Target.bBuildEditor == true)
            {
                PrivateDependencyModuleNames.AddRange(
                    new string[]
                    {
						"BlueprintGraph",
						"EditorFramework",
						"Kismet",
                        "UnrealEd",
                    }
                );
            }
        }
	}
}
