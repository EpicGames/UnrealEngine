// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraph/AnimGraphNode_EvaluationNotifies.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "AnimationWarping"

UAnimGraphNode_EvaluationNotifies::UAnimGraphNode_EvaluationNotifies(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
}

FText UAnimGraphNode_EvaluationNotifies::GetControllerDescription() const
{
	return LOCTEXT("EvaluationNotifies", "EvaluationNotifies");
}

FText UAnimGraphNode_EvaluationNotifies::GetTooltipText() const
{
	return LOCTEXT("EvaluationNotifiesTooltip", "Rotates the root and lower body by the specified angle, while counter rotating the upper body to maintain the forward facing direction.");
}

FText UAnimGraphNode_EvaluationNotifies::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return GetControllerDescription();
}

FLinearColor UAnimGraphNode_EvaluationNotifies::GetNodeTitleColor() const
{
	return FLinearColor(FColor(153.f, 0.f, 0.f));
}

void UAnimGraphNode_EvaluationNotifies::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);
}

void UAnimGraphNode_EvaluationNotifies::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::CustomizeDetails(DetailBuilder);
}

void UAnimGraphNode_EvaluationNotifies::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_FUNC()
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UAnimGraphNode_EvaluationNotifies::GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

void UAnimGraphNode_EvaluationNotifies::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::IAnimRootMotionProvider::AttributeName);
}

void UAnimGraphNode_EvaluationNotifies::ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog)
{
	Super::ValidateAnimNodeDuringCompilation(ForSkeleton, MessageLog);
}

#undef LOCTEXT_NAMESPACE
