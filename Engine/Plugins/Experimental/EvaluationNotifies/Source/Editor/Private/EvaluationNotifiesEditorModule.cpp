// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationNotifiesEditorModule.h"
#include "Modules/ModuleManager.h"

class FEvaluationNotifiesEditorModule : public IEvaluationNotifiesEditorModule
{
};


IMPLEMENT_MODULE( FEvaluationNotifiesEditorModule, EvaluationNotifiesEditor )
