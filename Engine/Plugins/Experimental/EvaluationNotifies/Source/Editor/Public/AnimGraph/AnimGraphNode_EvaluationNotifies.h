// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "AnimGraphNode_SkeletalControlBase.h"
#include "EvaluationNotifies/AnimNode_EvaluationNotifies.h"
#include "AnimGraphNode_EvaluationNotifies.generated.h"

namespace ENodeTitleType { enum Type : int; }

UCLASS()
class EVALUATIONNOTIFIESEDITOR_API UAnimGraphNode_EvaluationNotifies : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Settings)
	FAnimNode_EvaluationNotifies Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void GetInputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	virtual void ValidateAnimNodeDuringCompilation(USkeleton* ForSkeleton, FCompilerResultsLog& MessageLog) override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_Base interface
	virtual void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UAnimGraphNode_Base interface

	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};
