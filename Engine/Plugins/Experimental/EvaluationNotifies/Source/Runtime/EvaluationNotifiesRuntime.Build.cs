// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class EvaluationNotifiesRuntime : ModuleRules
	{
		public EvaluationNotifiesRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.Add(Path.Combine(EngineDirectory,"Source/ThirdParty/AHEasing/AHEasing-1.3.2"));
			
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"AnimationCore",
                "AnimGraphRuntime",
                "Core",
                "CoreUObject",
                "Engine",
                "AnimationWarpingRuntime",
                "RigVM",
                "UAF",
				"UAFAnimGraph", 
            });
		}
	}
}
