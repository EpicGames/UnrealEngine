// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNextAlignment.h"

#include "AnimNextAnimGraphSettings.h"
#include <AHEasing/easing.h>

#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimSequence.h"
#include "Engine/World.h"
#include "EvaluationNotifies/AnimNotifyState_Alignment.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "VisualLogger/VisualLogger.h"

void FEvaluationNotify_AlignmentInstance::Start()
{
	const UNotifyState_AlignmentBase* AlignmentNotify = CastChecked<UNotifyState_AlignmentBase>(AnimNotify);
	AlignBone = AlignmentNotify->AlignBone;
	bFirstFrame = true;
}

namespace UE::UAF::Alignment
{
	constexpr float FrameTime = 1.0f/30.0f;
	
	float SampleCurve(float SampleTime, float StartTime, float EndTime, const TArray<float>& CurveData)
	{
		if (SampleTime <= StartTime)
		{
			return 0.0f;
		}
		if (SampleTime >= EndTime)
		{
			return 1.0;
		}
	
		// do nearest neighbor sampling for simplicity and performance
		const int StartFrame = FMath::Clamp(round(StartTime / FrameTime), 0, CurveData.Num()-1);
		const int EndFrame = FMath::Clamp(round(EndTime / FrameTime), 0, CurveData.Num()-1);
		const int SampleFrame = FMath::Clamp(round(SampleTime / FrameTime), 0, CurveData.Num()-1);

		return (CurveData[SampleFrame] - CurveData[StartFrame])/(CurveData[EndFrame] - CurveData[StartFrame]);
	}
	
	void GetTransformForFrame(float Frame, const TArray<FTransform>& Trajectory, FTransform& OutTransform)
	{
		int32 LowerFrame = FMath::Clamp(floor(Frame), 0, Trajectory.Num() -1);
		int32 UpperFrame = FMath::Min(Trajectory.Num() -1 , LowerFrame + 1);
		float Alpha = Frame - LowerFrame;

		OutTransform = Trajectory[LowerFrame];
		OutTransform.BlendWith(Trajectory[UpperFrame], Alpha);
	}

}

float FEvaluationNotify_AlignmentInstance::GetWeight(float Time, const FAlignmentWarpCurve& WarpCurve) const 
{
	float Weight;

	if (WarpCurve.CurveType == EAlignmentWeightCurveType::FromRootMotionTranslation && !AnimTrajectoryData.TranslationCurve.IsEmpty())
	{
		const float Duration = (RoundedEndTime - StartTime);
		const float StartCurveSampleTime = FMath::Max(WarpCurve.StartRatio * Duration, ActualStartTime) - ActualStartTime;
		return UE::UAF::Alignment::SampleCurve(Time - ActualStartTime, StartCurveSampleTime, WarpCurve.EndRatio * Duration - ActualStartTime, AnimTrajectoryData.TranslationCurve);
	}
	else if (WarpCurve.CurveType == EAlignmentWeightCurveType::FromRootMotionRotation && !AnimTrajectoryData.RotationCurve.IsEmpty())
	{
		const float Duration = (RoundedEndTime - StartTime);
		const float StartCurveSampleTime = FMath::Max(WarpCurve.StartRatio * Duration, ActualStartTime) - ActualStartTime;
		return UE::UAF::Alignment::SampleCurve(Time - ActualStartTime, StartCurveSampleTime, WarpCurve.EndRatio * Duration - ActualStartTime, AnimTrajectoryData.RotationCurve);
	}
	else
	{
		const float StartTimeWithRatio = FMath::Max(FMath::Lerp(StartTime, RoundedEndTime, WarpCurve.StartRatio), ActualStartTime);
		const float EndTimeWithRatio = FMath::Lerp(StartTime, RoundedEndTime, WarpCurve.EndRatio);

		if(EndTimeWithRatio<=StartTimeWithRatio)
		{
			// the animation likely started late, and the blend ratio is finished so we should snap
			return 1.0f;
		}
		
		const float CurrentRelativeTime = FMath::Clamp((Time-StartTimeWithRatio)/(EndTimeWithRatio-StartTimeWithRatio), 0,1);
		
		switch(WarpCurve.CurveType)
		{
			case EAlignmentWeightCurveType::EaseIn:
				Weight = CubicEaseIn(CurrentRelativeTime);
				break;
			case EAlignmentWeightCurveType::EaseOut:
				Weight = CubicEaseOut(CurrentRelativeTime);
				break;
			case EAlignmentWeightCurveType::EaseInOut:
				Weight = CubicEaseInOut(CurrentRelativeTime);
				break;
			case EAlignmentWeightCurveType::Instant:
				Weight = 1.f;
				break;
			case EAlignmentWeightCurveType::DoNotWarp:
				Weight = 0.f;
				break;
			default:
				Weight = CurrentRelativeTime;
		}
	}

	
	return Weight;
}

void FEvaluationNotify_AlignToGroundInstance::End(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData)
{
	FEvaluationNotify_AlignmentInstance::End(TraitInstanceData);
	
	const UNotifyState_AlignToGround* AlignToGroundNotify = CastChecked<UNotifyState_AlignToGround>(AnimNotify);
	TraitInstanceData.Instance->SetVariable(AlignToGroundNotify->PlaybackRateOutputVariableReference, 1.0);
}

bool FEvaluationNotify_AlignToGroundInstance::GetTargetTransform(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData, FTransform& Transform)
{
	const UNotifyState_AlignToGround* AlignToGroundNotify = CastChecked<UNotifyState_AlignToGround>(AnimNotify);
	const FCollisionShape CollisionShape = FCollisionShape::MakeSphere(AlignToGroundNotify->TraceRadius);

	FTransform StartTransform = TraitInstanceData.RootBoneTransform;
	if (AlignToGroundNotify->bForceStartTransformUpright)
	{
		FRotator Rotation = StartTransform.GetRotation().Rotator();
		Rotation.Roll = 0;
		Rotation.Pitch = 0;
		StartTransform.SetRotation(Rotation.Quaternion());
	}
	
	const float PredictionDelta = FMath::Max(RoundedEndTime - CurrentTime, 0);
	FTransform PredictedRootMotion = TraitInstanceData.OnExtractRootMotionAttribute.Execute(CurrentTime, PredictionDelta, false);
	FTransform EndTransform = PredictedRootMotion * StartTransform;

	const FVector TraceDirectionWS = FVector::UpVector;
	const FVector TraceStart = EndTransform.GetLocation() + (TraceDirectionWS * AlignToGroundNotify->TraceStartOffset);
	const FVector TraceEnd = EndTransform.GetLocation() + (AlignToGroundNotify->TraceEndOffset * TraceDirectionWS);

	FCollisionQueryParams QueryParams;
	// Ignore self and all attached components
	// QueryParams.AddIgnoredActor(Context.OwningActor);

	const ECollisionChannel CollisionChannel = UEngineTypes::ConvertToCollisionChannel(AlignToGroundNotify->TraceChannel);

	if (const UObject* HostObject = TraitInstanceData.HostObject)
	{
		UWorld* World = HostObject->GetWorld();
		check(World);

		FHitResult HitResult;
		const bool bHit = World->SweepSingleByChannel(
		HitResult, TraceStart, TraceEnd, FQuat::Identity, CollisionChannel, CollisionShape, QueryParams);

		UE_VLOG_CAPSULE(TraitInstanceData.HostObject, "AlignToGround", Display,
					TraceEnd,
					(AlignToGroundNotify->TraceStartOffset - AlignToGroundNotify->TraceEndOffset) * 0.5f,
					AlignToGroundNotify->TraceRadius,
					FQuat::Identity,
					FColor::Green,
					TEXT(""));

		if (bHit)
		{
			Transform = EndTransform;
			Transform.SetLocation(HitResult.ImpactPoint + FVector(0,0,AlignToGroundNotify->GroundHeightOffset));
		
			static const FBox UnitBox(FVector(-10, -10, -10), FVector(10, 10, 10));
			UE_VLOG_OBOX(TraitInstanceData.HostObject, "AlignToGround", Display, UnitBox, Transform.ToMatrixWithScale(), FColor::Red, TEXT(""));

			if (!AlignToGroundNotify->PlaybackRateOutputVariableReference.IsNone())
			{
				float AnimatedFallDistance = StartTransform.GetTranslation().Z - EndTransform.GetTranslation().Z;
				
				if (AnimatedFallDistance > UE_KINDA_SMALL_NUMBER && PredictionDelta > UE_KINDA_SMALL_NUMBER)
				{
					// Math computes acceleration due to gravity as animated,
					// and then uses that to determine a modified playback rate for different actual fall heights (such that at the animated height the modifier will be 1.0)
					// - assuming starting from 0 vertical velocity (which is typically where you would want to start this notify, as you start falling)
					
					float AnimatedFallingAcceleration = 2.0f * AnimatedFallDistance / (PredictionDelta * PredictionDelta);
				
					float ActualFallDistance = StartTransform.GetTranslation().Z - HitResult.ImpactPoint.Z;
					float ModifiedTime = sqrtf(2.0f * ActualFallDistance/AnimatedFallingAcceleration);
				
					float PlaybackRate = FMath::Clamp(PredictionDelta / ModifiedTime, AlignToGroundNotify->MinPlaybackRateModifier, AlignToGroundNotify->MaxPlaybackRateModifier);
				
					TraitInstanceData.Instance->SetVariable(AlignToGroundNotify->PlaybackRateOutputVariableReference, (double)PlaybackRate);
				}
			}
			
			return true;
		}
	}

	return false;
}

FTransform GetModelSpaceTransform(const UE::UAF::FLODPoseStack& Pose, FBoneIndexType SkeletonBoneIndex)
{
	const TArrayView<const FBoneIndexType> SkeletonToPoseIndexMap = Pose.GetSkeletonBoneIndexToLODBoneIndexMap();
	const FBoneIndexType PoseIndex = SkeletonToPoseIndexMap[SkeletonBoneIndex];
	FTransform BoneTransform = Pose.LocalTransforms[PoseIndex];
						
	FBoneIndexType ParentSkeletonIndex = Pose.GetSkeletonAsset()->GetReferenceSkeleton().GetParentIndex(SkeletonBoneIndex);
	while(ParentSkeletonIndex != (FBoneIndexType)-1)
	{
		const FBoneIndexType ParentPoseIndex = SkeletonToPoseIndexMap[ParentSkeletonIndex];
		BoneTransform = BoneTransform * Pose.LocalTransforms[ParentPoseIndex];
		ParentSkeletonIndex = Pose.GetSkeletonAsset()->GetReferenceSkeleton().GetParentIndex(ParentSkeletonIndex);
	}

	return BoneTransform;
}

bool FEvaluationNotify_AlignmentInstance::GetTargetTransform(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData, FTransform& Transform)
{
	const UNotifyState_Alignment* AlignmentNotify = CastChecked<UNotifyState_Alignment>(AnimNotify);
	// TODO: this should be a proper variable reference held on the notify, but currently this uses a 'named transform' system too. Needs resolving
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return TraitInstanceData.Instance->GetVariable(FAnimNextVariableReference(AlignmentNotify->TransformName), Transform) == EPropertyBagResult::Success;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FEvaluationNotify_AlignmentInstance::Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData, UE::UAF::FEvaluationVM& VM)
{
	if (bDisabled)
	{
		return;
	}
	
	UNotifyState_AlignmentBase* AlignmentNotify = CastChecked<UNotifyState_AlignmentBase>(AnimNotify);

	if (const TUniquePtr<UE::UAF::FKeyframeState>* Keyframe = VM.PeekValue<TUniquePtr<UE::UAF::FKeyframeState>>(UE::UAF::KEYFRAME_STACK_NAME, 0))
	{
		if (bFirstFrame)
		{
			bFirstFrame = false;
			
			if (AlignmentNotify->Disable != NAME_None)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				TraitInstanceData.Instance->GetVariable(FAnimNextVariableReference(AlignmentNotify->Disable), bDisabled);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				
				if (bDisabled)
				{
					return;
				}
			}
			
			ActualStartTime = CurrentTime;
			RoundedEndTime = ActualStartTime + FMath::CeilToDouble((EndTime - ActualStartTime) / UE::UAF::Alignment::FrameTime) * UE::UAF::Alignment::FrameTime;
			
			const UE::UAF::FLODPoseStack& Pose = Keyframe->Get()->Pose;
			AlignBone.Initialize(Pose.GetSkeletonAsset());

			if (GetTargetTransform(TraitInstanceData, TargetTransform))
			{
				static const FBox UnitBox(FVector(-10, -10, -10), FVector(10, 10, 10));
				UE_VLOG_OBOX(TraitInstanceData.HostObject, "Alignment", Display, UnitBox, TargetTransform.ToMatrixWithScale(), FColor::Blue, TEXT(""));
			
				// get alignment transform
				const float PredictionDelta = RoundedEndTime - CurrentTime;
				
				TargetTransform = AlignmentNotify->AlignOffset * TargetTransform;

				if (AlignBone.HasValidSetup())
				{
					// get alignment bone relative to predicted root motion end point, and remove that as an offset to the alignment target
					// Alignment bone is expected to be at the alignment end point at the beginning of the Notify window  (we don't have a good way to predict the component space transform of it in the future)
				 	
					FTransform PredictedRootMotion = TraitInstanceData.OnExtractRootMotionAttribute.Execute(CurrentTime, PredictionDelta, false);

					FTransform AlignBoneTransform = GetModelSpaceTransform(Pose, AlignBone.BoneIndex);
				 	TargetTransform = PredictedRootMotion * AlignBoneTransform.Inverse() * TargetTransform;
				 }

				if (AlignmentNotify->bForceTargetTransformUpright)
				{
					FRotator Rotation = TargetTransform.GetRotation().Rotator();
					Rotation.Roll = 0;
					Rotation.Pitch = 0;
					TargetTransform.SetRotation(Rotation.Quaternion());
				}
			
				StartingRootTransform = TraitInstanceData.RootBoneTransform;
			
				// extract root motion trajectory todo: this should be cached and reused
			
				int NumFrames = 1 + (RoundedEndTime - ActualStartTime) / UE::UAF::Alignment::FrameTime;

				if (NumFrames <= 0)
				{
					return;
				}

				AnimTrajectoryData.Trajectory.SetNum(NumFrames);
				AnimTrajectoryData.TranslationCurve.SetNum(NumFrames);
				AnimTrajectoryData.RotationCurve.SetNum(NumFrames);
			
				float SteeringAngleThreshold = FMath::DegreesToRadians(AlignmentNotify->SteeringSettings.AngleThreshold);
			
				FTransform PredictedTransform;
				float PredictionTime = CurrentTime;
				
				AnimTrajectoryData.Trajectory[0] = PredictedTransform;
				PredictionTime += UE::UAF::Alignment::FrameTime;
				AnimTrajectoryData.TranslationCurve[0] = 0;

				for(int i=1;i<NumFrames; i++)
				{
					FTransform RootMotionThisFrame = TraitInstanceData.OnExtractRootMotionAttribute.Execute(PredictionTime, UE::UAF::Alignment::FrameTime, false);
				
					PredictedTransform = RootMotionThisFrame * PredictedTransform;
					AnimTrajectoryData.Trajectory[i] = PredictedTransform;
					PredictionTime += UE::UAF::Alignment::FrameTime;

					AnimTrajectoryData.TranslationCurve[i] = RootMotionThisFrame.GetTranslation().Length() + AnimTrajectoryData.TranslationCurve[i-1]; 
					AnimTrajectoryData.RotationCurve[i] = fabs(RootMotionThisFrame.GetRotation().GetAngle()) + AnimTrajectoryData.RotationCurve[i-1]; 
				}

				// normalize curves;
				if (AnimTrajectoryData.TranslationCurve.Last() > UE_SMALL_NUMBER)
				{
					if (AnimTrajectoryData.TranslationCurve.Last() < UE_SMALL_NUMBER)
					{
						AnimTrajectoryData.TranslationCurve.Reset();
					}
					else
					{
						for(float& Value : AnimTrajectoryData.TranslationCurve)
						{
							Value /= AnimTrajectoryData.TranslationCurve.Last();
						}
					}
				}
			
				if (AnimTrajectoryData.RotationCurve.Last() > UE_SMALL_NUMBER)
				{
					if (AnimTrajectoryData.RotationCurve.Last() < UE_SMALL_NUMBER)
					{
						AnimTrajectoryData.RotationCurve.Reset();
					}
					else
					{
						for(float& Value : AnimTrajectoryData.RotationCurve)
						{
							Value /= AnimTrajectoryData.RotationCurve.Last();
						}
					}
				}

				FVector UnWarpedPreviousPosition;
				FVector WarpedPreviousPosition;

				if (AlignmentNotify->bEnableSteering && AlignmentNotify->SteeringSettings.bEnableSmoothing)
				{
					FilteredSteeringTarget = FQuat::Identity;
					TargetSmoothingState.Reset();

					FilteredSteeringTarget = UKismetMathLibrary::QuaternionSpringInterp(FilteredSteeringTarget, FQuat::Identity, TargetSmoothingState,
													AlignmentNotify->SteeringSettings.SmoothStiffness, AlignmentNotify->SteeringSettings.SmoothDamping, TraitInstanceData.DeltaTime, 1, 0, true);
				}

				WarpedTrajectory.SetNum(AnimTrajectoryData.Trajectory.Num());
				
				FVector MovementDirection = TraitInstanceData.RootBoneTransform.GetRotation().GetRightVector(); // todo, get this from translation over first few frames or based on some setting?
			
				FTransform InverseLastFrame = AnimTrajectoryData.Trajectory.Last().Inverse();
				// Translation warping + steering
				for(int i=0;i<NumFrames; i++)
				{
					FTransform TransformFromRoot = AnimTrajectoryData.Trajectory[i] * TraitInstanceData.RootBoneTransform;
					FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
				
					FVector OldPosition = TransformFromRoot.GetTranslation();
					FVector UnWarpedDelta = OldPosition - UnWarpedPreviousPosition;
					UnWarpedPreviousPosition = OldPosition;

					FVector NewPosition = OldPosition;
					
					if (AlignmentNotify->bSeparateTranslationCurves)
					{
						float InMovementDirectionWeight = GetWeight(ActualStartTime + UE::UAF::Alignment::FrameTime*i, AlignmentNotify->TranslationWarpingCurve_InMovementDirection);
						float OutOfMovementDirectionWeight = GetWeight(ActualStartTime + UE::UAF::Alignment::FrameTime*i, AlignmentNotify->TranslationWarpingCurve_OutOfMovementDirection);
	
						FVector Delta = TransformFromTarget.GetTranslation() - OldPosition;
						FVector InMovementDirectionDelta = MovementDirection * Delta.Dot(MovementDirection);
						FVector OutOfMovementDirectionDelta = Delta - InMovementDirectionDelta;
	
						NewPosition = OldPosition + InMovementDirectionDelta * InMovementDirectionWeight + OutOfMovementDirectionDelta * OutOfMovementDirectionWeight;
					}
					else
					{
						float Weight = GetWeight(ActualStartTime + UE::UAF::Alignment::FrameTime*i, AlignmentNotify->TranslationWarpingCurve);
						NewPosition = FMath::Lerp(OldPosition, TransformFromTarget.GetTranslation(), Weight);
					}

					FVector WarpedDelta = NewPosition - WarpedPreviousPosition;
					WarpedPreviousPosition = NewPosition;

					WarpedTrajectory[i].SetTranslation(NewPosition);
					WarpedTrajectory[i].SetRotation(TransformFromRoot.GetRotation());

					if (i > 0 && AlignmentNotify->bEnableSteering)
					{
						FQuat OldRotation = TransformFromRoot.GetRotation();
						FQuat DirectionChange = FQuat::FindBetweenVectors(FVector(UnWarpedDelta.X, UnWarpedDelta.Y, 0) , FVector(WarpedDelta.X, WarpedDelta.Y, 0));

						if (AlignmentNotify->SteeringSettings.bEnableSmoothing)
						{
						
							if (DirectionChange.GetAngle() < SteeringAngleThreshold)
							{
								FilteredSteeringTarget = UKismetMathLibrary::QuaternionSpringInterp(FilteredSteeringTarget, DirectionChange, TargetSmoothingState,
																AlignmentNotify->SteeringSettings.SmoothStiffness, AlignmentNotify->SteeringSettings.SmoothDamping, TraitInstanceData.DeltaTime, 1, 0, true);
							}
                        
							DirectionChange = FilteredSteeringTarget;
							WarpedTrajectory[i].SetRotation(OldRotation*DirectionChange);
						}
						else
						{
							if (DirectionChange.GetAngle() < SteeringAngleThreshold)
							{
								WarpedTrajectory[i].SetRotation(OldRotation*DirectionChange);
							}
						}
					
					}
				}
			
				// Rotation warping
				for(int i=0;i<NumFrames; i++)
				{
					float Weight = GetWeight(ActualStartTime + UE::UAF::Alignment::FrameTime*i, AlignmentNotify->RotationWarpingCurve);
				
					FQuat OldRotation = WarpedTrajectory[i].GetRotation();
					FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
				
					WarpedTrajectory[i].SetRotation(FQuat::Slerp(OldRotation, TransformFromTarget.GetRotation(), Weight));
				}
			
			}
		}
	
		const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
		ensureMsgf(RootMotionProvider, TEXT("Alignment expected a valid root motion delta provider interface."));
    		
		if (RootMotionProvider && !WarpedTrajectory.IsEmpty())
		{
			float Frame = (CurrentTime + TraitInstanceData.DeltaTime - ActualStartTime) / UE::UAF::Alignment::FrameTime;
			
			if (PreviousFrame < WarpedTrajectory.Num() - 1)
			{
				FTransform WorldTransform;
				
				if (Frame > WarpedTrajectory.Num()-1)
				{
					float PostAlignmentTime = (Frame - (WarpedTrajectory.Num() - 1)) * UE::UAF::Alignment::FrameTime;
					FTransform PostAlignmentRootMotion = TraitInstanceData.OnExtractRootMotionAttribute.Execute(RoundedEndTime, PostAlignmentTime, false);
					UE::UAF::Alignment::GetTransformForFrame(Frame, WarpedTrajectory, WorldTransform);
					WorldTransform = PostAlignmentRootMotion * WarpedTrajectory.Last();
				}
				else
				{
					UE::UAF::Alignment::GetTransformForFrame(Frame, WarpedTrajectory, WorldTransform);
				}

				if(AlignmentNotify->UpdateMode == EAlignmentUpdateMode::World)
				{
					RootMotionProvider->OverrideRootMotion(WorldTransform.GetRelativeTransform(TraitInstanceData.RootBoneTransform), Keyframe->Get()->Attributes);
				}
				else // relative mode
				{
					FTransform PrevTransform;
					UE::UAF::Alignment::GetTransformForFrame(PreviousFrame, WarpedTrajectory, PrevTransform);
					RootMotionProvider->OverrideRootMotion(WorldTransform.GetRelativeTransform(PrevTransform), Keyframe->Get()->Attributes);
				}
		
				// unwarped trajectory relative to starting transform
				FTransform PreviousTransform = AnimTrajectoryData.Trajectory[0] * StartingRootTransform;
				for(int i=1;i<AnimTrajectoryData.Trajectory.Num(); i++)
				{
					FTransform TransformFromRoot = AnimTrajectoryData.Trajectory[i] * StartingRootTransform;
					UE_VLOG_SEGMENT(TraitInstanceData.HostObject, "Alignment", Display, PreviousTransform.GetLocation(), TransformFromRoot.GetLocation(), i%2 == 0 ? FColor::Yellow : FColor::Red, TEXT(""));
					PreviousTransform = TransformFromRoot;
				}
		
				// unwarped trajectory relative to target transform
				FTransform InverseLastFrame = AnimTrajectoryData.Trajectory.Last().Inverse();
				PreviousTransform = AnimTrajectoryData.Trajectory[0] * InverseLastFrame * TargetTransform;
				for(int i=1;i<AnimTrajectoryData.Trajectory.Num(); i++)
				{
					FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
					UE_VLOG_SEGMENT(TraitInstanceData.HostObject, "Alignment", Display, PreviousTransform.GetLocation(), TransformFromTarget.GetLocation(), i%2 == 0 ? FColor::Yellow : FColor::Red, TEXT(""));
					PreviousTransform = TransformFromTarget;
				}
		
				// the warped trajectory
				for(int i=1; i<WarpedTrajectory.Num(); i++)
				{
					UE_VLOG_SEGMENT(TraitInstanceData.HostObject, "Alignment", Display, WarpedTrajectory[i-1].GetLocation(), WarpedTrajectory[i].GetLocation(), i%2 == 0 ? FColor::Green : FColor::Blue, TEXT(""));
				}

				// a dot representing our current root position
				UE_VLOG_SPHERE(TraitInstanceData.HostObject, "Alignment", Display, TraitInstanceData.RootBoneTransform.GetLocation(),  1, FColor::Green, TEXT(""));
				
				// a dot representing our current target position on the trajectory 
				UE_VLOG_SPHERE(TraitInstanceData.HostObject, "Alignment", Display, WorldTransform.GetLocation(),  1, FColor::Red, TEXT(""));
				
				PreviousFrame = Frame;
			}
		}
	}
}

