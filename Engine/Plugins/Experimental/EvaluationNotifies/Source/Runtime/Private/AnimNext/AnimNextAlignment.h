// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNext/EvaluationNotifiesTrait.h"
#include "AnimNextAlignment.generated.h"

struct FAlignmentWarpCurve;

struct AlignmentAnimTrajectoryData
{
	FTransform TargetTransform;
	TArray<FTransform> Trajectory;
	TArray<float> TranslationCurve;
	TArray<float> RotationCurve;
};


USTRUCT()
struct FEvaluationNotify_AlignmentInstance : public FEvaluationNotify_BaseInstance
{
	GENERATED_BODY()
	virtual void Start() override;
	virtual bool GetTargetTransform(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData, FTransform& TargetTranform);
   	virtual void Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData, UE::UAF::FEvaluationVM& VM) override;
	
	float GetWeight(float Time, const FAlignmentWarpCurve& WarpCurve) const; 

	FBoneReference AlignBone;
	bool bDisabled = false;
	bool bFirstFrame = true;
	float ActualStartTime = 0;
	float RoundedEndTime = 0;
	float PreviousFrame = 0;

	FTransform StartingRootTransform;
	FTransform TargetTransform;
	
	FQuat FilteredSteeringTarget = FQuat::Identity;
	FQuaternionSpringState TargetSmoothingState;

	TArray<FTransform> WarpedTrajectory;
	AlignmentAnimTrajectoryData AnimTrajectoryData;
};

USTRUCT()
struct FEvaluationNotify_AlignToGroundInstance : public FEvaluationNotify_AlignmentInstance
{
	GENERATED_BODY()
	virtual void End(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData) override;
	virtual bool GetTargetTransform(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& TraitInstanceData, FTransform& TargetTranform);
};