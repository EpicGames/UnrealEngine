// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNext/EvaluationNotifiesTrait.h"

#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "TraitInterfaces/INotifySource.h"
#include "TraitInterfaces/ITimeline.h"
#include "TraitCore/NodeInstance.h"

static const FName RootBoneTransformName = "RootBoneTransform";

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FEvaluationNotifiesTrait

namespace UE::UAF
{
	TMap<UClass*, UScriptStruct*> FEvaluationNotifiesTrait::NotifyEvaluationHandlerMap;

	AUTO_REGISTER_ANIM_TRAIT(FEvaluationNotifiesTrait)

	#define TRAIT_INTERFACE_ENUMERATOR(GeneratorMacro) \
		GeneratorMacro(ITimelinePlayer) \
		GeneratorMacro(IUpdate) \
		GeneratorMacro(IEvaluate) \

	// Trait required interfaces implementation boilerplate
    #define TRAIT_REQUIRED_INTERFACE_ENUMERATOR(GeneratorMacroRequired) \
    	GeneratorMacroRequired(ITimelinePlayer) \
		GeneratorMacroRequired(ITimeline) \
    	GeneratorMacroRequired(INotifySource) \

	// Trait implementation boilerplate
	GENERATE_ANIM_TRAIT_IMPLEMENTATION(FEvaluationNotifiesTrait, TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_INTERFACE_ENUMERATOR, NULL_ANIM_TRAIT_EVENT_ENUMERATOR)
	#undef TRAIT_INTERFACE_ENUMERATOR

	void FEvaluationNotifiesTrait::AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const
	{
		TTraitBinding<ITimeline> TimelineTrait;
		ensure(Binding.GetStackInterfaceSuper(TimelineTrait));

		TTraitBinding<ITimelinePlayer> TimelinePlayerTrait;
		ensure(Binding.GetStackInterfaceSuper(TimelinePlayerTrait));

		// Get current state from the stack, advance time, then get the delta state
		FTimelineState PreAdvanceState;
		const bool bHasState = TimelineTrait.GetState(Context, PreAdvanceState);

		TimelinePlayerTrait.AdvanceBy(Context, DeltaTime, bDispatchEvents);

		if (!bDispatchEvents || !bHasState)
		{
			return;
		}

		TTraitBinding<INotifySource> NotifySourceTrait;
		ensure(Binding.GetStackInterfaceSuper(NotifySourceTrait));

		FTimelineDelta Delta;
		ensure(TimelineTrait.GetDelta(Context, Delta));

		// Query for notifies
		TArray<FAnimNotifyEventReference> Notifies;
		NotifySourceTrait.GetNotifies(Context, PreAdvanceState.GetPosition(), Delta.GetDeltaTime(), PreAdvanceState.IsLooping(), Notifies);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		InstanceData->DeltaTime = Delta.GetDeltaTime();

		for(const FAnimNotifyEventReference& NotifyData : Notifies)
		{
			if(const FAnimNotifyEvent* NotifyEvent = NotifyData.GetNotify())
			{
				if (UAnimNotifyState* Notify = Cast<UAnimNotifyState>(NotifyEvent->NotifyStateClass))
				{
					if (UScriptStruct** HandlerType = NotifyEvaluationHandlerMap.Find(Notify->GetClass()))
					{
						if (FInstancedStruct* FoundInstance = InstanceData->EvaluationNotifies.FindByPredicate([NotifyEvent](const FStructView ExistingInstance)
						{
							// check if we already have an instance for this event
							return ExistingInstance.Get<FEvaluationNotify_BaseInstance>().NotifyEvent == NotifyEvent;
						}))
						{
							FoundInstance->GetMutable<FEvaluationNotify_BaseInstance>().CurrentTime = NotifyData.GetCurrentAnimationTime();
						}
						else
						{
							FInstancedStruct& Struct = InstanceData->EvaluationNotifies.AddDefaulted_GetRef();
							Struct.InitializeAs(*HandlerType);
							FEvaluationNotify_BaseInstance& Instance = Struct.GetMutable<FEvaluationNotify_BaseInstance>();
							Instance.NotifyEvent = NotifyEvent;
							Instance.AnimNotify = Notify;
							Instance.StartTime = NotifyEvent->GetTriggerTime();
							Instance.EndTime = NotifyEvent->GetEndTriggerTime();
							Instance.CurrentTime = NotifyData.GetCurrentAnimationTime();
						}
					}
				}
			}
		}
	}

	void FEvaluationNotifiesTrait::PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const
	{
		IEvaluate::PostEvaluate(Context, Binding);

		const FSharedData* SharedData = Binding.GetSharedData<FSharedData>();
		check(SharedData);

		FInstanceData* InstanceData = Binding.GetInstanceData<FInstanceData>();
		check(InstanceData);

		InstanceData->HostObject = Context.GetHostObject();

		// Get Root motion provider
		{
			TTraitBinding<IAttributeProvider> AttributeTrait;
			if (!Binding.GetStackInterface(AttributeTrait))
			{
				// UE_LOGF(LogAnimNextWarping, Error, "FSteeringTrait::PostEvaluate, missing IAttributeProvider");
				return;
			}

			InstanceData->OnExtractRootMotionAttribute = AttributeTrait.GetOnExtractRootMotionAttribute(Context);
		}
		
		InstanceData->Instance = &Binding.GetTraitPtr().GetNodeInstance()->GetOwner();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InstanceData->Instance->GetVariable(FAnimNextVariableReference(RootBoneTransformName), InstanceData->RootBoneTransform);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		Context.AppendTask(FAnimNextEvaluationNotifiesTask::Make(InstanceData, SharedData));
	}

	void FEvaluationNotifiesTrait::RegisterEvaluationHandler(UClass* NotifyType, UScriptStruct* HandlerType)
	{
		NotifyEvaluationHandlerMap.Add(NotifyType, HandlerType);
	}

	void FEvaluationNotifiesTrait::UnregisterEvaluationHandler(UClass* NotifyType)
	{
		NotifyEvaluationHandlerMap.Remove(NotifyType);
	}

	void FEvaluationNotifiesTrait::FInstanceData::Destruct(const FExecutionContext& Context, const FTraitBinding& Binding)
	{
		for (FStructView NotifyInstanceData : Binding.GetInstanceData<FInstanceData>()->EvaluationNotifies)
		{
			FEvaluationNotify_BaseInstance& NotifyInstance = NotifyInstanceData.Get<FEvaluationNotify_BaseInstance>();
			if (NotifyInstance.bActive)
			{
				NotifyInstance.bActive = false;
				NotifyInstance.End(*this);
			}
		}

		FTrait::FInstanceData::Destruct(Context, Binding);
	}

} // namespace UE::UAF


////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// FAnimNextEvaluationNotifiesTask

FAnimNextEvaluationNotifiesTask FAnimNextEvaluationNotifiesTask::Make(UE::UAF::FEvaluationNotifiesTrait::FInstanceData* InstanceData, const UE::UAF::FEvaluationNotifiesTrait::FSharedData* SharedData)
{
	FAnimNextEvaluationNotifiesTask Task;
	Task.InstanceData = InstanceData;
	Task.SharedData = SharedData;
	return Task;
}

void FAnimNextEvaluationNotifiesTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;

	// Execute EvaluationNotifies
	if (InstanceData->DeltaTime > 0)
	{
		for(auto& NotifyInstanceData : InstanceData->EvaluationNotifies)
		{
			FEvaluationNotify_BaseInstance& NotifyInstance = NotifyInstanceData.GetMutable<FEvaluationNotify_BaseInstance>();
			
			if (NotifyInstance.StartTime <= NotifyInstance.CurrentTime && NotifyInstance.EndTime >= NotifyInstance.CurrentTime)
			{
				if (!NotifyInstance.bActive)
				{
					NotifyInstance.bActive = true;
					NotifyInstance.Start();
				}

				NotifyInstance.Update(*InstanceData, VM);
			}
			else
			{
				if (NotifyInstance.bActive)
				{
					NotifyInstance.Update(*InstanceData, VM);
					NotifyInstance.bActive = false;
					NotifyInstance.End(*InstanceData);
				}
			}

			NotifyInstance.CurrentTime += InstanceData->DeltaTime;
		}

		InstanceData->EvaluationNotifies.RemoveAll([](const FInstancedStruct& Data)
		{
			return Data.Get<FEvaluationNotify_BaseInstance>().bActive == false;
		});
	
	}

}

