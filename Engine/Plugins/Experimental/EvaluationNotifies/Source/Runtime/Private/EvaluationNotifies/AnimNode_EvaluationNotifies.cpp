// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationNotifies/AnimNode_EvaluationNotifies.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimRootMotionProvider.h"
#include "Animation/AnimTrace.h"
#include "BoneControllers/AnimNode_OffsetRootBone.h"
#include "Curves/BezierUtilities.h"
#include "StructUtils/InstancedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_EvaluationNotifies)

TMap<UClass*, UScriptStruct*> FAnimNode_EvaluationNotifies::NotifyEvaluationHandlerMap;

void FAnimNode_EvaluationNotifies::GetEvaluationNotifiesFromAnimation(const UAnimSequenceBase* Animation, TArray<FInstancedStruct>& OutNotifyInstances)
{
	if(Animation)
	{
		// todo: we should probably preprocess sequences for this number
		int32 Count = 0;
		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (UAnimNotifyState* Notify = Cast<UAnimNotifyState>(NotifyEvent.NotifyStateClass))
			{
				if (UScriptStruct** EvaluationHandlerStruct = NotifyEvaluationHandlerMap.Find(Notify->GetClass()))
				{
					Count++;
				}
			}
		}

		OutNotifyInstances.Empty(Count);

		for (int32 Idx = 0; Idx < Animation->Notifies.Num(); Idx++)
		{
			const FAnimNotifyEvent& NotifyEvent = Animation->Notifies[Idx];
			if (UAnimNotifyState* Notify = Cast<UAnimNotifyState>(NotifyEvent.NotifyStateClass))
			{
				if (UScriptStruct** EvaluationHandlerStruct = NotifyEvaluationHandlerMap.Find(Notify->GetClass()))
				{
					FInstancedStruct& Struct = OutNotifyInstances.AddDefaulted_GetRef();
					Struct.InitializeAs(*EvaluationHandlerStruct);
					FEvaluationNotifyInstance& Instance = Struct.GetMutable<FEvaluationNotifyInstance>();
					Instance.AnimNotify = Notify;
					Instance.StartTime = NotifyEvent.GetTriggerTime();
					Instance.EndTime = NotifyEvent.GetEndTriggerTime();
				}
			}
		}
	}
}

void FAnimNode_EvaluationNotifies::UpdateInternal(const FAnimationUpdateContext& Context)
{
	FAnimNode_SkeletalControlBase::UpdateInternal(Context);
	
	const bool bJustBecameRelevant = !UpdateCounter.WasSynchronizedCounter(Context.AnimInstanceProxy->GetUpdateCounter());
	UpdateCounter.SynchronizeWith(Context.AnimInstanceProxy->GetUpdateCounter());

	if (bJustBecameRelevant)
	{
		// clear state on become relevant in case there's any left over evaluation notifies from last time
		CurrentSequence = nullptr;
		Tags.Empty();
	}

	TRACE_ANIM_NODE_VALUE(Context, TEXT("Asset"), CurrentAnimAsset);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Playback Time"), CurrentAnimAssetTime);
	TRACE_ANIM_NODE_VALUE(Context, TEXT("Mirrored"), CurrentAnimAssetMirrored);

	if (UE::AnimationWarping::FRootOffsetProvider* RootOffsetProvider = Context.GetMessage<UE::AnimationWarping::FRootOffsetProvider>())
	{
		RootBoneTransform = RootOffsetProvider->GetRootTransform();
	}
	else
	{
		RootBoneTransform = Context.AnimInstanceProxy->GetComponentTransform();
	}
}

void FAnimNode_EvaluationNotifies::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);
}

void FAnimNode_EvaluationNotifies::GatherDebugData(FNodeDebugData& DebugData)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)

	FString DebugLine = DebugData.GetNodeName(this);
	// Just tracking the animation asset for now. Tracking the map of notifies being evaluated would be better suited in rewind debugger.
	DebugLine += FString::Printf(TEXT("(Animation: %s, Animation Time: %.3f"),
		*CurrentAnimAsset.GetName(),
		CurrentAnimAssetTime
		);

	ComponentPose.GatherDebugData(DebugData);
}


void FAnimNode_EvaluationNotifies::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	const float DeltaSeconds = Output.AnimInstanceProxy->GetDeltaSeconds();

	if (DeltaSeconds > 0.f)
	{
		const UAnimSequenceBase* AnimSequenceBase = Cast<UAnimSequenceBase>(CurrentAnimAsset);
		if (AnimSequenceBase != CurrentSequence)
		{
			PreviousAnimAssetTime = CurrentAnimAssetTime;
			CurrentSequence = AnimSequenceBase;
			GetEvaluationNotifiesFromAnimation(CurrentSequence, Tags);
		}

		for(auto& TagData : Tags)
		{
			FEvaluationNotifyInstance& Tag = TagData.GetMutable<FEvaluationNotifyInstance>();
			if (Tag.StartTime <= CurrentAnimAssetTime && Tag.EndTime >= CurrentAnimAssetTime)
			{
				if (!Tag.bActive)
				{
					Tag.bActive = true;
					Tag.Start(CurrentSequence);
				}

				Tag.Update(CurrentSequence, CurrentAnimAssetTime, DeltaSeconds, CurrentAnimAssetMirrored, MirrorDataTable, RootBoneTransform, NamedTransforms, Output, OutBoneTransforms);
			}
			else
			{
				if (Tag.bActive)
				{
					Tag.bActive = false;
					Tag.End();
				}
			}
			
		}
	}
}

void FAnimNode_EvaluationNotifies::RegisterEvaluationHandler(UClass* NotifyType, UScriptStruct* HandlerType)
{
	NotifyEvaluationHandlerMap.Add(NotifyType, HandlerType);
}

void FAnimNode_EvaluationNotifies::UnregisterEvaluationHandler(UClass* NotifyType)
{
	NotifyEvaluationHandlerMap.Remove(NotifyType);
}
