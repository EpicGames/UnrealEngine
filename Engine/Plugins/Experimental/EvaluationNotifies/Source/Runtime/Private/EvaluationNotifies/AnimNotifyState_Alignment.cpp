// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationNotifies/AnimNotifyState_Alignment.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimRootMotionProvider.h"
#include "AHEasing/easing.h"
#include "Animation/AnimSequenceHelpers.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_Alignment)

namespace
{
	constexpr float FrameTime = 1.0f/30.0f;

	FTransform ExtractRootMotionHelper(const UAnimSequenceBase* AnimationAsset, const UMirrorDataTable* MirrorDataTable, bool bIsMirrored, float CurrentTime, float DeltaTime, bool bLoop)
	{
		if (bIsMirrored)
		{
			return UE::Anim::ExtractRootMotionFromAnimationAsset(AnimationAsset, MirrorDataTable, CurrentTime, DeltaTime, bLoop);
		}
		else
		{
			const FAnimExtractContext Context(static_cast<double>(CurrentTime), true, FDeltaTimeRecord(DeltaTime), bLoop);
			return AnimationAsset->ExtractRootMotion(Context);
		}
	}
	
	void GetTransformForFrame(float Frame, const TArray<FTransform>& Trajectory, FTransform& OutTransform)
	{
		int32 LowerFrame = FMath::Clamp(floor(Frame), 0, Trajectory.Num() -1);
		int32 UpperFrame = FMath::Min(Trajectory.Num() -1 , LowerFrame + 1);
		float Alpha = Frame - LowerFrame;

		OutTransform = Trajectory[LowerFrame];
		OutTransform.BlendWith(Trajectory[UpperFrame], Alpha);
	}
}

void FAlignmentNotifyInstance::Start(const UAnimSequenceBase* AnimationAsset)
{
	UNotifyState_Alignment* AlignmentNotify = Cast<UNotifyState_Alignment>(AnimNotify);
	AlignBone = AlignmentNotify->AlignBone;
	bFirstFrame = true;
}

float SampleCurve(float SampleTime, float StartTime, float EndTime, const TArray<float>& CurveData)
{
	if (SampleTime <= StartTime)
	{
		return 0.0f;
	}
	if (SampleTime >= EndTime)
	{
		return 1.0;
	}
	
	// do nearest neighbor sampling for simplicity and performance
	const int StartFrame = FMath::Clamp(round(StartTime / FrameTime), 0, CurveData.Num()-1);
	const int EndFrame = FMath::Clamp(round(EndTime / FrameTime), 0, CurveData.Num()-1);
	const int SampleFrame = FMath::Clamp(round(SampleTime / FrameTime), 0, CurveData.Num()-1);

	return (CurveData[SampleFrame] - CurveData[StartFrame])/(CurveData[EndFrame] - CurveData[StartFrame]);
}




float FAlignmentNotifyInstance::GetWeight(float CurrentTime, const FAlignmentWarpCurve& WarpCurve) const 
{
	UNotifyState_Alignment* AlignmentNotify = Cast<UNotifyState_Alignment>(AnimNotify);
	float Weight;
	

	if (WarpCurve.CurveType == EAlignmentWeightCurveType::FromRootMotionTranslation && !AnimTrajectoryData.TranslationCurve.IsEmpty())
	{
		const float Duration = (EndTime - StartTime);
		const float StartCurveSampleTime = FMath::Max(WarpCurve.StartRatio * Duration, ActualStartTime) - ActualStartTime;
		return SampleCurve(CurrentTime - ActualStartTime, StartCurveSampleTime, WarpCurve.EndRatio * Duration - ActualStartTime, AnimTrajectoryData.TranslationCurve);
	}
	else if (WarpCurve.CurveType == EAlignmentWeightCurveType::FromRootMotionRotation && !AnimTrajectoryData.RotationCurve.IsEmpty())
	{
		const float Duration = (EndTime - StartTime);
		const float StartCurveSampleTime = FMath::Max(WarpCurve.StartRatio * Duration, ActualStartTime) - ActualStartTime;
		return SampleCurve(CurrentTime - ActualStartTime, StartCurveSampleTime, WarpCurve.EndRatio * Duration - ActualStartTime, AnimTrajectoryData.RotationCurve);
	}
	else
	{
		const float StartTimeWithRatio = FMath::Max(FMath::Lerp(StartTime, EndTime, WarpCurve.StartRatio), ActualStartTime);
		const float EndTimeWithRatio = FMath::Lerp(StartTime, EndTime, WarpCurve.EndRatio);
			
		float CurrentRelativeTime;
		if (StartTimeWithRatio < EndTimeWithRatio - UE_SMALL_NUMBER)
		{
			CurrentRelativeTime = FMath::Clamp((CurrentTime - StartTimeWithRatio) / (EndTimeWithRatio - StartTimeWithRatio), 0, 1);
		}
		else
		{
			CurrentRelativeTime = 1.f;
		}
		
		switch(WarpCurve.CurveType)
		{
			case EAlignmentWeightCurveType::EaseIn:
				Weight = CubicEaseIn(CurrentRelativeTime);
				break;
			case EAlignmentWeightCurveType::EaseOut:
				Weight = CubicEaseOut(CurrentRelativeTime);
				break;
			case EAlignmentWeightCurveType::EaseInOut:
				Weight = CubicEaseInOut(CurrentRelativeTime);
				break;
			case EAlignmentWeightCurveType::Instant:
				Weight = 1.f;
				break;
			case EAlignmentWeightCurveType::DoNotWarp:
				Weight = 0.f;
				break;
			default:
				Weight = CurrentRelativeTime;
		}
	}

	
	return Weight;
}


void FAlignmentNotifyInstance::Update(const UAnimSequenceBase* AnimationAsset, float CurrentTime, float DeltaTime, bool bIsMirrored, const UMirrorDataTable* MirrorDataTable,
                                   FTransform& RootBoneTransform, const TMap<FName, FTransform>& NamedTransforms, FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	UNotifyState_Alignment* AlignmentNotify = Cast<UNotifyState_Alignment>(AnimNotify);
	
	if (bFirstFrame)
	{
		bFirstFrame = false;
		ActualStartTime = CurrentTime;
		
		const FBoneContainer& RequiredBones = Output.AnimInstanceProxy->GetRequiredBones();	
		AlignBone.Initialize(RequiredBones);
		
		if (const FTransform* FoundTransform = NamedTransforms.Find(AlignmentNotify->TransformName))
		{
			TargetTransform = *FoundTransform;
			
			// get alignment transform
			const float PredictionDelta = EndTime - CurrentTime ;
			
			if (AlignBone.HasValidSetup())
			{
				// get alignment bone relative to predicted root motion end point, and remove that as an offset to the alignment target
				// Alignment bone is expected to be at the alignment end point at the beginning of the Notify window  (we don't have a good way to prediuct the component space transform of it in the future)
				
				FTransform PredictedRootMotion = ExtractRootMotionHelper(AnimationAsset, MirrorDataTable, bIsMirrored, CurrentTime, PredictionDelta, false);
				FCompactPoseBoneIndex AlignBoneIndex = AlignBone.GetCompactPoseIndex(RequiredBones);
				if (AlignBoneIndex.IsValid())
				{
					TargetTransform = Output.Pose.GetComponentSpaceTransform(AlignBoneIndex).Inverse() * PredictedRootMotion * TargetTransform;
				}
			}
			
			TargetTransform = AlignmentNotify->AlignOffset * TargetTransform;

			static const FBox UnitBox(FVector(-10, -10, -10), FVector(10, 10, 10));
			UE_VLOG_OBOX(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Alignment", Display, UnitBox, TargetTransform.ToMatrixWithScale(), FColor::Blue, TEXT(""));
			
			StartingRootTransform = RootBoneTransform;
			
			// extract root motion trajectory todo: this should be cached and reused
			
			int NumFrames = (EndTime - CurrentTime) / FrameTime;

			AnimTrajectoryData.Trajectory.SetNum(NumFrames);
			AnimTrajectoryData.TranslationCurve.SetNum(NumFrames);
			AnimTrajectoryData.RotationCurve.SetNum(NumFrames);
			
			float SteeringAngleThreshold = FMath::DegreesToRadians(AlignmentNotify->SteeringSettings.AngleThreshold);
			
			FTransform PredictedTransform;
			float PredictionTime = CurrentTime;
			float TargetTime = EndTime;

			for(int i=0;i<NumFrames; i++)
			{
				FVector PrevPosition = PredictedTransform.GetLocation();
				FTransform RootMotionThisFrame = ExtractRootMotionHelper(AnimationAsset, MirrorDataTable, bIsMirrored, PredictionTime, FrameTime, false);
				
				PredictedTransform = RootMotionThisFrame * PredictedTransform;
				AnimTrajectoryData.Trajectory[i] = PredictedTransform;
				PredictionTime += FrameTime;

				AnimTrajectoryData.TranslationCurve[i] = RootMotionThisFrame.GetTranslation().Length();
				if (i>0)
				{
					AnimTrajectoryData.TranslationCurve[i] += AnimTrajectoryData.TranslationCurve[i-1]; 
				}
				
				AnimTrajectoryData.RotationCurve[i] = fabs(RootMotionThisFrame.GetRotation().GetAngle());
				if (i>0)
				{
					AnimTrajectoryData.RotationCurve[i] += AnimTrajectoryData.RotationCurve[i-1]; 
				}
			}

			// normalize curves;
			if (AnimTrajectoryData.TranslationCurve.Last() > UE_SMALL_NUMBER)
			{
				if (AnimTrajectoryData.TranslationCurve.Last() < UE_SMALL_NUMBER)
				{
					AnimTrajectoryData.TranslationCurve.Reset();
				}
				else
				{
					for(float& Value : AnimTrajectoryData.TranslationCurve)
					{
						Value /= AnimTrajectoryData.TranslationCurve.Last();
					}
				}
			}
			
			if (AnimTrajectoryData.RotationCurve.Last() > UE_SMALL_NUMBER)
			{
				if (AnimTrajectoryData.RotationCurve.Last() < UE_SMALL_NUMBER)
				{
					AnimTrajectoryData.RotationCurve.Reset();
				}
				else
				{
					for(float& Value : AnimTrajectoryData.RotationCurve)
					{
						Value /= AnimTrajectoryData.RotationCurve.Last();
					}
				}
			}

			FVector UnWarpedPreviousPosition;
			FVector WarpedPreviousPosition;

			if (AlignmentNotify->bEnableSteering && AlignmentNotify->SteeringSettings.bEnableSmoothing)
			{
				FilteredSteeringTarget = FQuat::Identity;
				TargetSmoothingState.Reset();
			}

			WarpedTrajectory.SetNum(AnimTrajectoryData.Trajectory.Num());
			
			FTransform InverseLastFrame = AnimTrajectoryData.Trajectory.Last().Inverse();
			// Translation warping + steering
			for(int i=0;i<NumFrames; i++)
			{
				float Weight = GetWeight(ActualStartTime + FrameTime*i, AlignmentNotify->TranslationWarpingCurve);

				FTransform TransformFromRoot = AnimTrajectoryData.Trajectory[i] * RootBoneTransform;
				FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
				
				FVector OldPosition = TransformFromRoot.GetTranslation();
				FVector UnWarpedDelta = OldPosition - UnWarpedPreviousPosition;
				UnWarpedPreviousPosition = OldPosition;
				
				FVector NewPosition = FMath::Lerp(OldPosition, TransformFromTarget.GetTranslation(), Weight);
				FVector WarpedDelta = NewPosition - WarpedPreviousPosition;
				WarpedPreviousPosition = NewPosition;

				WarpedTrajectory[i].SetTranslation(NewPosition);
				WarpedTrajectory[i].SetRotation(TransformFromRoot.GetRotation());

				if (i > 0 && AlignmentNotify->bEnableSteering)
				{
					FQuat OldRotation = TransformFromRoot.GetRotation();
					FQuat DirectionChange = FQuat::FindBetweenVectors(UnWarpedDelta, WarpedDelta);

					if (AlignmentNotify->SteeringSettings.bEnableSmoothing)
					{
						
						if (DirectionChange.GetAngle() < SteeringAngleThreshold)
						{
							FilteredSteeringTarget = UKismetMathLibrary::QuaternionSpringInterp(FilteredSteeringTarget, DirectionChange, TargetSmoothingState,
															AlignmentNotify->SteeringSettings.SmoothStiffness, AlignmentNotify->SteeringSettings.SmoothDamping, DeltaTime, 1, 0, true);
						}
                        
                        DirectionChange = FilteredSteeringTarget;
						WarpedTrajectory[i].SetRotation(OldRotation*DirectionChange);
					}
					else
					{
						if (DirectionChange.GetAngle() < SteeringAngleThreshold)
						{
							WarpedTrajectory[i].SetRotation(OldRotation*DirectionChange);
						}
					}
				}
			}
			
			// Rotation warping
			for(int i=0;i<NumFrames; i++)
			{
				float Weight = GetWeight(ActualStartTime + FrameTime*i, AlignmentNotify->RotationWarpingCurve);
				
				FQuat OldRotation = WarpedTrajectory[i].GetRotation();
				FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
				
				WarpedTrajectory[i].SetRotation(FQuat::Slerp(OldRotation, TransformFromTarget.GetRotation(), Weight));
			}
		}
	}
	
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
	ensureMsgf(RootMotionProvider, TEXT("Alignment expected a valid root motion delta provider interface."));
	
    		
	if (RootMotionProvider && !WarpedTrajectory.IsEmpty())
	{
		float Frame = (CurrentTime - ActualStartTime) / FrameTime;

		int32 LowerFrame = FMath::Clamp(floor(Frame), 0, WarpedTrajectory.Num() -1);
		int32 UpperFrame = FMath::Min(WarpedTrajectory.Num() -1 , LowerFrame + 1);
		float Alpha = Frame - LowerFrame;
		

		FTransform WorldTransform = WarpedTrajectory[LowerFrame];
		WorldTransform.BlendWith(WarpedTrajectory[UpperFrame], Alpha);

		if(AlignmentNotify->UpdateMode == EAlignmentUpdateMode::World)
		{
			RootMotionProvider->OverrideRootMotion(WorldTransform.GetRelativeTransform(RootBoneTransform), Output.CustomAttributes);
		}
		else // relative mode
		{
			FTransform PrevTransform;
			GetTransformForFrame(PreviousFrame, WarpedTrajectory, PrevTransform);
			RootMotionProvider->OverrideRootMotion(WorldTransform.GetRelativeTransform(PrevTransform), Output.CustomAttributes);
			PreviousFrame = Frame;
		}
		
		// unwarped trajectory relative to starting transform
		FTransform PreviousTransform = AnimTrajectoryData.Trajectory[0] * StartingRootTransform;
		for(int i=1;i<AnimTrajectoryData.Trajectory.Num(); i++)
		{
			FTransform TransformFromRoot = AnimTrajectoryData.Trajectory[i] * StartingRootTransform;
			UE_VLOG_SEGMENT(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Alignment", Display, PreviousTransform.GetLocation(), TransformFromRoot.GetLocation(), i%2 == 0 ? FColor::Yellow : FColor::Red, TEXT(""));
			PreviousTransform = TransformFromRoot;
		}
		
		// unwarped trajectory relative to target transform
		FTransform InverseLastFrame = AnimTrajectoryData.Trajectory.Last().Inverse();
		PreviousTransform = AnimTrajectoryData.Trajectory[0] * InverseLastFrame * TargetTransform;
		for(int i=1;i<AnimTrajectoryData.Trajectory.Num(); i++)
		{
			FTransform TransformFromTarget = AnimTrajectoryData.Trajectory[i] * InverseLastFrame * TargetTransform;
			UE_VLOG_SEGMENT(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Alignment", Display, PreviousTransform.GetLocation(), TransformFromTarget.GetLocation(), i%2 == 0 ? FColor::Yellow : FColor::Red, TEXT(""));
			PreviousTransform = TransformFromTarget;
		}
		
		// the warped trajectory
		for(int i=1; i<WarpedTrajectory.Num(); i++)
		{
			UE_VLOG_SEGMENT(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Alignment", Display, WarpedTrajectory[i-1].GetLocation(), WarpedTrajectory[i].GetLocation(), i%2 == 0 ? FColor::Green : FColor::Blue, TEXT(""));
		}

		// a dot representing our current position on the trajectory 
		UE_VLOG_SPHERE(Output.AnimInstanceProxy->GetAnimInstanceObject(), "Alignment", Display, WorldTransform.GetLocation(),  1, FColor::Red, TEXT(""));
	}
}

void UNotifyState_AlignToGround::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::AnimNextVariableReferences)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PlaybackRateOutputVariableReference = FAnimNextVariableReference(PlaybackRateOutputVariable_DEPRECATED);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
}
