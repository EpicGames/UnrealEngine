// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationNotifies/AnimNotifyState_TwoBoneIK.h"
#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimRootMotionProvider.h"
#include "HAL/IConsoleManager.h"
#include "TwoBoneIK.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNotifyState_TwoBoneIK)

void FTwoBoneIKNotifyInstance::Start(const UAnimSequenceBase* AnimationAsset)
{
	UNotifyState_TwoBoneIK* TwoBoneIKNotify = Cast<UNotifyState_TwoBoneIK>(AnimNotify);
	IKBone = TwoBoneIKNotify->IKBone;
	RelativeToBone = TwoBoneIKNotify->RelativeToBone;
	EffectorTarget = TwoBoneIKNotify->EffectorTarget;
	JointTarget = TwoBoneIKNotify->JointTarget;
}

FTransform GetTargetTransform(const FTransform& InComponentTransform, FCSPose<FCompactPose>& MeshBases, FBoneSocketTarget& InTarget, EBoneControlSpace Space, const FVector& InOffset) 
{
	FTransform OutTransform;
	if (Space == BCS_BoneSpace)
	{
		OutTransform = InTarget.GetTargetTransform(InOffset, MeshBases, InComponentTransform);
	}
	else
	{
		// parent bone space still goes through this way
		// if your target is socket, it will try find parents of joint that socket belongs to
		OutTransform.SetLocation(InOffset);
		FAnimationRuntime::ConvertBoneSpaceTransformToCS(InComponentTransform, MeshBases, OutTransform, InTarget.GetCompactPoseBoneIndex(), Space);
	}

	return OutTransform;
}

bool g_bEnableTwoBoneIKNotify = true;
static FAutoConsoleVariableRef CVarEnableTwoBoneIKNotify(
	TEXT("Animation.Notify.TwoBoneIK.Enable"),
	g_bEnableTwoBoneIKNotify,
	TEXT("Enable Two BoneIK Notify"));

void FTwoBoneIKNotifyInstance::Update(const UAnimSequenceBase* AnimationAsset, float CurrentTime, float DeltaTime, bool bIsMirrored, const UMirrorDataTable* MirrorDataTable,
                                   FTransform& InRootBoneTransform, const TMap<FName, FTransform>& NamedTransforms, FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	if (!g_bEnableTwoBoneIKNotify)
	{
		return;
	}
	
	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	
	const UE::Anim::IAnimRootMotionProvider* RootMotionProvider = UE::Anim::IAnimRootMotionProvider::Get();
    ensureMsgf(RootMotionProvider, TEXT("Alignment expected a valid root motion delta provider interface."));

	FTransform RootBoneTransform = InRootBoneTransform;
	FTransform ThisFrameRootMotionTransform;
	if (RootMotionProvider->ExtractRootMotion(Output.CustomAttributes, ThisFrameRootMotionTransform))
	{
		RootBoneTransform =  ThisFrameRootMotionTransform * RootBoneTransform;
	}
	
	UNotifyState_TwoBoneIK* Data = Cast<UNotifyState_TwoBoneIK>(AnimNotify);
	check(Data);

	if (!IKBone.IsValidToEvaluate())
	{
		IKBone.Initialize(BoneContainer);
		RelativeToBone.Initialize(BoneContainer);

		EffectorTarget.InitializeBoneReferences(BoneContainer);
		JointTarget.InitializeBoneReferences(BoneContainer);

		FCompactPoseBoneIndex IKBoneCompactPoseIndex = IKBone.GetCompactPoseIndex(BoneContainer);
		CachedLowerLimbIndex = FCompactPoseBoneIndex(INDEX_NONE);
		CachedUpperLimbIndex = FCompactPoseBoneIndex(INDEX_NONE);
		if (IKBoneCompactPoseIndex != INDEX_NONE)
		{
			CachedLowerLimbIndex = BoneContainer.GetParentBoneIndex(IKBoneCompactPoseIndex);
			if (CachedLowerLimbIndex != INDEX_NONE)
			{
				CachedUpperLimbIndex = BoneContainer.GetParentBoneIndex(CachedLowerLimbIndex);
			}
		}
	}

	// Get indices of the lower and upper limb bones and check validity.
	bool bInvalidLimb = false;

	FCompactPoseBoneIndex IKBoneCompactPoseIndex = IKBone.GetCompactPoseIndex(BoneContainer);
	FCompactPoseBoneIndex RelativeToBoneCompactPoseIndex = RelativeToBone.GetCompactPoseIndex(BoneContainer);

	const bool bInBoneSpace = (Data->EffectorLocationSpace == BCS_ParentBoneSpace) || (Data->EffectorLocationSpace == BCS_BoneSpace);

	// Get Local Space transforms for our bones. We do this first in case they already are local.
	// As right after we get them in component space. (And that does the auto conversion).
	// We might save one transform by doing local first...
	const FTransform EndBoneLocalTransform = Output.Pose.GetLocalSpaceTransform(IKBoneCompactPoseIndex);
	const FTransform LowerLimbLocalTransform = Output.Pose.GetLocalSpaceTransform(CachedLowerLimbIndex);
	const FTransform UpperLimbLocalTransform = Output.Pose.GetLocalSpaceTransform(CachedUpperLimbIndex);

	// Now get those in component space...
	FTransform LowerLimbCSTransform = Output.Pose.GetComponentSpaceTransform(CachedLowerLimbIndex);
	FTransform UpperLimbCSTransform = Output.Pose.GetComponentSpaceTransform(CachedUpperLimbIndex);
	FTransform EndBoneCSTransform = Output.Pose.GetComponentSpaceTransform(IKBoneCompactPoseIndex);

	// Get current position of root of limb.
	// All position are in Component space.
	const FVector RootPos = UpperLimbCSTransform.GetTranslation();
	const FVector InitialJointPos = LowerLimbCSTransform.GetTranslation();
	const FVector InitialEndPos = EndBoneCSTransform.GetTranslation();

	FVector EffectorLocation = Data->EffectorLocation;
	if (const FTransform* EffectorLocationTransform = NamedTransforms.Find(Data->EffectorLocationTransformName))
	{
		EffectorLocation = EffectorLocationTransform->GetLocation();
	}

	// // Transform EffectorLocation from EffectorLocationSpace to ComponentSpace.
	// FTransform EffectorTransform = GetTargetTransform(RootBoneTransform, Output.Pose, Data->EffectorTarget, Data->EffectorLocationSpace, EffectorLocation);
	FTransform EffectorTransform;
	EffectorTransform.SetLocation(EffectorLocation);
	EffectorTransform = EffectorTransform.GetRelativeTransform(RootBoneTransform);

	if(RelativeToBoneCompactPoseIndex.IsValid())
	{
		// if RelativeToBone was set, then compute relative position offset.
		// - add component space position difference to effect position
		
		const FTransform RelativeToTransform = Output.Pose.GetComponentSpaceTransform(RelativeToBoneCompactPoseIndex);
		const FVector Offset = EndBoneCSTransform.GetLocation() - RelativeToTransform.GetLocation();
		EffectorTransform.SetLocation(EffectorTransform.GetLocation() + Offset);
	}
	
	// blending.  For now just blending the effector location
	float Weight = 1.0f;
	if (Data->BlendInTime > 0.0f && CurrentTime - StartTime < Data->BlendInTime)
	{
		Weight = FMath::Max(CurrentTime - StartTime, 0)/Data->BlendInTime;
	}
	else if (Data->BlendOutTime > 0.0f && EndTime - StartTime < Data->BlendOutTime)
	{
		Weight = 1.0f - FMath::Max(EndTime - CurrentTime, 0)/Data->BlendInTime;
	}

	EffectorTransform.BlendWith(EndBoneCSTransform, 1.0f-Weight);

	// Get joint target (used for defining plane that joint should be in).
	FTransform JointTargetTransform = GetTargetTransform(RootBoneTransform, Output.Pose, Data->JointTarget, Data->JointTargetLocationSpace,  Data->JointTargetLocation);

	FVector	JointTargetPos = JointTargetTransform.GetTranslation();

	UE_VLOG_SPHERE(Output.AnimInstanceProxy->GetAnimInstanceObject(),"TwoBoneIK", Display, RootBoneTransform.TransformPosition(JointTargetPos), 0.1f, FColor::Red, TEXT(""));

	// This is our reach goal.
	FVector DesiredPos = EffectorTransform.GetTranslation();

	// IK solver
	UpperLimbCSTransform.SetLocation(RootPos);
	LowerLimbCSTransform.SetLocation(InitialJointPos);
	EndBoneCSTransform.SetLocation(InitialEndPos);

	
	AnimationCore::SolveTwoBoneIK(UpperLimbCSTransform, LowerLimbCSTransform, EndBoneCSTransform, JointTargetPos, DesiredPos, Data->bAllowStretching, Data->StartStretchRatio, Data->MaxStretchScale);

// #if WITH_EDITOR
// 	CachedJointTargetPos = JointTargetPos;
// 	CachedJoints[0] = UpperLimbCSTransform.GetLocation();
// 	CachedJoints[1] = LowerLimbCSTransform.GetLocation();
// 	CachedJoints[2] = EndBoneCSTransform.GetLocation();
// #endif // WITH_EDITOR

	// if no twist, we clear twist from each limb
	if (!Data->bAllowTwist)
	{
		auto RemoveTwist = [this](const FTransform& InParentTransform, FTransform& InOutTransform, const FTransform& OriginalLocalTransform, const FVector& InAlignVector) 
		{
			FTransform LocalTransform = InOutTransform.GetRelativeTransform(InParentTransform);
			FQuat LocalRotation = LocalTransform.GetRotation();
			FQuat NewTwist, NewSwing;
			LocalRotation.ToSwingTwist(InAlignVector, NewSwing, NewTwist);
			NewSwing.Normalize();

			// get new twist from old local
			LocalRotation = OriginalLocalTransform.GetRotation();
			FQuat OldTwist, OldSwing;
			LocalRotation.ToSwingTwist(InAlignVector, OldSwing, OldTwist);
			OldTwist.Normalize();

			InOutTransform.SetRotation(InParentTransform.GetRotation() * NewSwing * OldTwist);
			InOutTransform.NormalizeRotation();
		};

		const FCompactPoseBoneIndex UpperLimbParentIndex = BoneContainer.GetParentBoneIndex(CachedUpperLimbIndex);
		FVector AlignDir = Data->TwistAxis.GetTransformedAxis(FTransform::Identity);
		if (UpperLimbParentIndex != INDEX_NONE)
		{
			FTransform UpperLimbParentTransform = Output.Pose.GetComponentSpaceTransform(UpperLimbParentIndex);
			RemoveTwist(UpperLimbParentTransform, UpperLimbCSTransform, UpperLimbLocalTransform, AlignDir);
		}
			
		RemoveTwist(UpperLimbCSTransform, LowerLimbCSTransform, LowerLimbLocalTransform, AlignDir);
	}


	
	// Update transform for upper bone.
	{
		// Order important. First bone is upper limb.
		OutBoneTransforms.Add( FBoneTransform(CachedUpperLimbIndex, UpperLimbCSTransform) );
	}

	// Update transform for lower bone.
	{
		// Order important. Second bone is lower limb.
		OutBoneTransforms.Add( FBoneTransform(CachedLowerLimbIndex, LowerLimbCSTransform) );
	}

	// Update transform for end bone.
	{
		// only allow bTakeRotationFromEffectorSpace during bone space
		if (bInBoneSpace && Data->bTakeRotationFromEffectorSpace)
		{
			EndBoneCSTransform.SetRotation(EffectorTransform.GetRotation());
		}
		else if (Data->bMaintainEffectorRelRot)
		{
			EndBoneCSTransform = EndBoneLocalTransform * LowerLimbCSTransform;
		}
		// Order important. Third bone is End Bone.
		OutBoneTransforms.Add(FBoneTransform(IKBoneCompactPoseIndex, EndBoneCSTransform));
		
	}

	// Make sure we have correct number of bones
	// check(OutBoneTransforms.Num() == 3);
}

