// Copyright Epic Games, Inc. All Rights Reserved.

#include "EvaluationNotifiesRuntimeModule.h"
#include "Animation/AnimRootMotionProvider.h"
#include "AnimNext/EvaluationNotifiesTrait.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

#include "AnimNext/AnimNextAlignment.h"
#include "EvaluationNotifies/AnimNotifyState_Alignment.h"
#include "EvaluationNotifies/AnimNotifyState_TwoBoneIK.h"

class FEvaluationNotifiesRuntimeModule : public IEvaluationNotifiesRuntimeModule
{
};

namespace UE
{
	namespace EvaluationNotifies
	{
		class FModule : public IModuleInterface
		{
		public:

			FModule();
	
			virtual void StartupModule() override;
			virtual void ShutdownModule() override;
		};

		FModule::FModule()
		{
		}

		void FModule::StartupModule()
		{
			FAnimNode_EvaluationNotifies::RegisterEvaluationHandler(UNotifyState_Alignment::StaticClass(), FAlignmentNotifyInstance::StaticStruct());
			FAnimNode_EvaluationNotifies::RegisterEvaluationHandler(UNotifyState_TwoBoneIK::StaticClass(), FTwoBoneIKNotifyInstance::StaticStruct());

			UE::UAF::FEvaluationNotifiesTrait::RegisterEvaluationHandler(UNotifyState_Alignment::StaticClass(), FEvaluationNotify_AlignmentInstance::StaticStruct());
			UE::UAF::FEvaluationNotifiesTrait::RegisterEvaluationHandler(UNotifyState_AlignToGround::StaticClass(), FEvaluationNotify_AlignToGroundInstance::StaticStruct());
		}

		void FModule::ShutdownModule()
		{
			FAnimNode_EvaluationNotifies::UnregisterEvaluationHandler(UNotifyState_Alignment::StaticClass());
			FAnimNode_EvaluationNotifies::UnregisterEvaluationHandler(UNotifyState_TwoBoneIK::StaticClass());
			
			UE::UAF::FEvaluationNotifiesTrait::UnregisterEvaluationHandler(UNotifyState_Alignment::StaticClass());
			UE::UAF::FEvaluationNotifiesTrait::UnregisterEvaluationHandler(UNotifyState_AlignToGround::StaticClass());
		}
	}
}
	

IMPLEMENT_MODULE(UE::EvaluationNotifies::FModule, EvaluationNotifiesRuntime)
