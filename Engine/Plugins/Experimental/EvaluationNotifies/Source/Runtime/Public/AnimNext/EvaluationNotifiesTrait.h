// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/KismetMathLibrary.h"
#include "TraitCore/Trait.h"
#include "TraitCore/TraitSharedData.h"
#include "TraitInterfaces/IAttributeProvider.h"
#include "TraitInterfaces/IEvaluate.h"
#include "TraitInterfaces/ITimelinePlayer.h"
#include "TraitInterfaces/IUpdate.h"
#include "EvaluationNotifiesTrait.generated.h"

struct FEvaluationNotify_BaseInstance;
struct FUAFAssetInstance;

/**
 * Data needed to execute steering
 * 
 * Some steering data such as current anim asset / playback time is acquired via trait stack interfaces
 */
USTRUCT(meta = (DisplayName = "EvaluationNotifies"))
struct FEvaluationNotifiesTraitSharedData : public FAnimNextTraitSharedData
{
	GENERATED_BODY()

	// Latent pin support boilerplate
	#define TRAIT_LATENT_PROPERTIES_ENUMERATOR(GeneratorMacro) \

	GENERATE_TRAIT_LATENT_PROPERTIES(FEvaluationNotifiesTraitSharedData, TRAIT_LATENT_PROPERTIES_ENUMERATOR)
	#undef TRAIT_LATENT_PROPERTIES_ENUMERATOR
};

namespace UE::UAF
{
/**
 * this trait will run evaluation time code for notifies which have a registered handler
 */
struct EVALUATIONNOTIFIESRUNTIME_API FEvaluationNotifiesTrait : FAdditiveTrait, ITimelinePlayer, IUpdate, IEvaluate
{
	DECLARE_ANIM_TRAIT(FEvaluationNotifiesTrait, FAdditiveTrait)

	using FSharedData = FEvaluationNotifiesTraitSharedData;

	struct FInstanceData : FTrait::FInstanceData
	{
		void Destruct(const FExecutionContext& Context, const FTraitBinding& Binding);

		/** Delta in seconds between updates, populated during PreUpdate */
		float DeltaTime = 0.f;

		/** Callback provided by attribute trait on stack to evaluate root motion at a later time */
		FOnExtractRootMotionAttribute OnExtractRootMotionAttribute = FOnExtractRootMotionAttribute();

		/** Last root bone transform sampled */
		FTransform RootBoneTransform = FTransform::Identity;

		/** the evaluation notifies extracted from the current anim sequence */
		TArray<FInstancedStruct> EvaluationNotifies;

		FUAFAssetInstance* Instance = nullptr;

		/** Owner object the trait instance is associated to */
		const UObject* HostObject;
	};

	// ITimelinePlayer impl
	virtual void AdvanceBy(FExecutionContext& Context, const TTraitBinding<ITimelinePlayer>& Binding, float DeltaTime, bool bDispatchEvents) const override;
	
	// IEvaluate impl
	virtual void PostEvaluate(FEvaluateTraversalContext& Context, const TTraitBinding<IEvaluate>& Binding) const override;

	static void RegisterEvaluationHandler(UClass* NotifyType, UScriptStruct* Handler);
	static void UnregisterEvaluationHandler(UClass* NotifyType);

private:
	static TMap<UClass*, UScriptStruct*> NotifyEvaluationHandlerMap;
};

} // namespace UE::UAF

/** Task to run EvaluationNotifies on VM */
USTRUCT()
struct FAnimNextEvaluationNotifiesTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FAnimNextEvaluationNotifiesTask)

	static FAnimNextEvaluationNotifiesTask Make(UE::UAF::FEvaluationNotifiesTrait::FInstanceData* InstanceData, const UE::UAF::FEvaluationNotifiesTrait::FSharedData* SharedData);

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UE::UAF::FEvaluationNotifiesTrait::FInstanceData* InstanceData = nullptr;
	const UE::UAF::FEvaluationNotifiesTrait::FSharedData* SharedData = nullptr;
};

USTRUCT()
struct FEvaluationNotify_BaseInstance
{
	GENERATED_BODY()
	virtual void Start() {}
	virtual void Update(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData, UE::UAF::FEvaluationVM& VM) {}
	virtual void End(UE::UAF::FEvaluationNotifiesTrait::FInstanceData& InstanceData) {}
	
	virtual ~FEvaluationNotify_BaseInstance() {};

	const FAnimNotifyEvent* NotifyEvent;
	TObjectPtr<UAnimNotifyState> AnimNotify = nullptr;
	float StartTime = 0.f;
	float EndTime = 0.f;
	
	float CurrentTime = 0.f;
	bool bActive = false;
};