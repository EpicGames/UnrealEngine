// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "BoneControllers/BoneControllerTypes.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "Kismet/KismetMathLibrary.h"
#include "StructUtils/InstancedStruct.h"
#include "AnimNode_EvaluationNotifies.generated.h"

struct FAnimationInitializeContext;
struct FComponentSpacePoseContext;
struct FNodeDebugData;
class UAnimationAsset;

USTRUCT()
struct FEvaluationNotifyInstance
{
	GENERATED_BODY()
	
	virtual void Start(const UAnimSequenceBase* AnimationAsset) {}
	virtual void Update(const UAnimSequenceBase* AnimationAsset, float CurrentTime, float DeltaTime, bool bIsMirrored, const UMirrorDataTable* MirrorDataTable,
		FTransform& RootBoneTransform, const TMap<FName, FTransform>& NamedTransforms, FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) {}
	virtual void End() {}

	virtual ~FEvaluationNotifyInstance() {}

	TObjectPtr<UAnimNotifyState> AnimNotify = nullptr;
	float StartTime = 0.f;
	float EndTime = 0.f;
	bool bActive = false;
};

// add procedural delta to the root motion attribute 
USTRUCT(BlueprintInternalUseOnly)
struct EVALUATIONNOTIFIESRUNTIME_API FAnimNode_EvaluationNotifies : public FAnimNode_SkeletalControlBase
{
	GENERATED_BODY()

	// Animation Asset for incorporating root motion data. If CurrentAnimAsset is set, and the animation has root motion rotation within the TargetTime, then those rotations will be scaled to reach the TargetOrientation
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	TObjectPtr<UAnimationAsset> CurrentAnimAsset;
	
	// Current playback time in seconds of the CurrentAnimAsset
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	float CurrentAnimAssetTime = 0.f;
	
	// Is the current anim asset mirrored
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	bool CurrentAnimAssetMirrored = false;
	
	// If bMirrored is set, MirrorDataTable will be used for mirroring the CurrentAnimAsset during prediction
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Evaluation, meta = (PinShownByDefault))
	TObjectPtr<UMirrorDataTable> MirrorDataTable;
	
	// Current playback time in seconds of the CurrentAnimAsset
	UPROPERTY(EditAnywhere, Transient, BlueprintReadWrite, Category=Evaluation, meta=(PinShownByDefault))
	TMap<FName, FTransform> NamedTransforms;

	// FAnimNodeBase interface
	virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNodeBase interface
	
	// FAnimNode_SkeletalControlBase interface
	virtual void UpdateInternal(const FAnimationUpdateContext& Context) override;
	virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override { return true; }
	// End of FAnimNode_SkeletalControlBase interface

	static void RegisterEvaluationHandler(UClass* NotifyType, UScriptStruct* Handler);
	static void UnregisterEvaluationHandler(UClass* NotifyType);

private:
	void GetEvaluationNotifiesFromAnimation(const UAnimSequenceBase* Animation, TArray<FInstancedStruct>& OutNotifyInstances);

	FTransform RootBoneTransform;
	TArray<FInstancedStruct> Tags;
	float PreviousAnimAssetTime = 0;
	UPROPERTY(Transient)
	TObjectPtr<const UAnimSequenceBase> CurrentSequence = nullptr;

	static TMap<UClass*, UScriptStruct*> NotifyEvaluationHandlerMap;

	// Update Counter for detecting being relevant
	FGraphTraversalCounter UpdateCounter;
};
