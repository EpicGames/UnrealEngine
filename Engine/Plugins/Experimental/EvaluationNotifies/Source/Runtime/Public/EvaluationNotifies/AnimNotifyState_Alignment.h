// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_EvaluationNotifies.h"
#include "Variables/AnimNextVariableReference.h"
#include "AnimNotifyState_Alignment.generated.h"

UENUM()
enum class EAlignmentWeightCurveType
{
	FromRootMotionTranslation,
	FromRootMotionRotation,
	Linear,
	EaseIn,
	EaseOut,
	EaseInOut,
	Instant,
	DoNotWarp,
};

USTRUCT()
struct FAlignmentWarpCurve
{
	GENERATED_BODY()
	
	// Type of curve to interpolate using
	UPROPERTY(EditAnywhere, Category = Properties)
	EAlignmentWeightCurveType CurveType = EAlignmentWeightCurveType::FromRootMotionTranslation;

	// Time to start warping, as a ratio of the duration of the Notify
	UPROPERTY(EditAnywhere, Category = "Properties", meta = (UIMin = "0.0", UIMax = "1.0", SliderExponent = "1.0"))
	float StartRatio = 0.f;

	// Time to complete warping, as a ratio of the duration of the Notify
	UPROPERTY(EditAnywhere, Category = "Properties", meta = (UIMin = "0.0", UIMax = "1.0", SliderExponent = "1.0"))
	float EndRatio = 1.f;
};

USTRUCT()
struct FAlignmentSteeringSettings
{
	GENERATED_BODY()

	// Enable smoothing of the steering target orientation, to avoid instant orientation changes
	UPROPERTY(EditAnywhere, Category=Properties)
	bool bEnableSmoothing = true;

	// Sprint stiffness for smoothing
	UPROPERTY(EditAnywhere, Category=Propeties, meta = (EditCondition=bEnableSmoothing))
	float SmoothStiffness = 300;

	// Spring damping factor for smoothing
	UPROPERTY(EditAnywhere, Category=Properties, meta = (EditCondition=bEnableSmoothing))
	float SmoothDamping = 1;
	
	// When the warped movement direction differs from the animated movement direction by more than this threshold, steering will be disabled.
	UPROPERTY(EditAnywhere, Category = Properties)
	float AngleThreshold = 135;
};

struct AnimTrajectoryData
{
	FTransform TargetTransform;
	TArray<FTransform> Trajectory;
	TArray<float> TranslationCurve;
	TArray<float> RotationCurve;
};

UENUM()
enum class EAlignmentUpdateMode
{
	// each frame, apply root motion to get to the expected part of the alignment path in wold space
	World,
	// each frame, apply the root motion from the alignment path relative to the previous frame
	Relative,
};

UCLASS(MinimalAPI)
class UNotifyState_AlignmentBase : public UAnimNotifyState
{
	GENERATED_BODY()
	
	public:

	UPROPERTY(EditAnywhere, Category = Blending)
	FAlignmentWarpCurve TranslationWarpingCurve;

	
	UPROPERTY(EditAnywhere, Category = Blending, Meta = (EditCondition="!bUseSeparateTranslationCurves"))
	bool bSeparateTranslationCurves;

	// curve for warping translation in direction of movement
	UPROPERTY(EditAnywhere, Category = Blending, Meta = (EditCondition="bUseSeparateTranslationCurves"))
	FAlignmentWarpCurve TranslationWarpingCurve_InMovementDirection;
	UPROPERTY(EditAnywhere, Category = Blending, Meta = (EditCondition="bUseSeparateTranslationCurves"))
	FAlignmentWarpCurve TranslationWarpingCurve_OutOfMovementDirection;
	
	UPROPERTY(EditAnywhere, Category = Blending)
	FAlignmentWarpCurve RotationWarpingCurve = { };
	
	// Offset from Root (or AlignBone) to Align to the target transform
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	FTransform AlignOffset;

	// only allow Yaw in the target transform (remove other rotations before warping)
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	bool bForceTargetTransformUpright;
	

#if WITH_EDITOR
	// For automating setting the AlignOffset at the end of the NotifyState
    UFUNCTION(CallInEditor, BlueprintCallable, Category = "TechAnimUtils")
    void SetAlignOffset(FTransform NewTransform)
    {
		AlignOffset = NewTransform;
    }
#endif

	// Optional bone to Align to target transform
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	FBoneReference AlignBone;
		
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	EAlignmentUpdateMode UpdateMode;
	
	// Bool variable which will disable this notify
	UPROPERTY(EditAnywhere, Category = TargetTransform)
	FName Disable;

	// Steering will rotate the character based on the difference between the animated movement direction and the warped movement direction, to keep the character facing their direction of movement
	UPROPERTY(EditAnywhere, Category = Steering)
	bool bEnableSteering = false;
	
	UPROPERTY(EditAnywhere, Category = Steering, meta=(EditCondition=bEnableSteering))
	FAlignmentSteeringSettings SteeringSettings;
};

UCLASS(MinimalAPI, BlueprintType, DisplayName="Alignment")
class UNotifyState_Alignment : public UNotifyState_AlignmentBase
{
	GENERATED_BODY()
	
	public:
	// Named transform to align to
    UPROPERTY(EditAnywhere, Category = TargetTransform)
    FName TransformName;

#if WITH_EDITOR
	UFUNCTION(CallInEditor, BlueprintCallable, Category = "TechAnimUtils")
	void SetTransformName(FName NewName)
	{
		TransformName = NewName;
	}
#endif
};

USTRUCT()
struct FAlignmentNotifyInstance : public FEvaluationNotifyInstance
{
	GENERATED_BODY()

	virtual void Start(const UAnimSequenceBase* AnimationAsset) override;
	virtual void Update(const UAnimSequenceBase* AnimationAsset, float CurrentTime, float DeltaTime, bool bIsMirrored, const UMirrorDataTable* MirrorDataTable,
		FTransform& RootBoneTransform, const TMap<FName, FTransform>& NamedTransforms, FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	
	float GetWeight(float CurrentTime, const FAlignmentWarpCurve& WarpCurve) const;
	
	FBoneReference AlignBone;
	bool bFirstFrame = true;
	float ActualStartTime = 0;
	float PreviousFrame = 0;

	FTransform StartingRootTransform;
	FTransform TargetTransform;
	
	FQuat FilteredSteeringTarget = FQuat::Identity;
	FQuaternionSpringState TargetSmoothingState;

	TArray<FTransform> WarpedTrajectory;
	AnimTrajectoryData AnimTrajectoryData;
};

UCLASS(BlueprintType, DisplayName="AlignToGround")
class UNotifyState_AlignToGround : public UNotifyState_AlignmentBase
{
	GENERATED_BODY()

	// UObject interface
	virtual void Serialize(FArchive& Ar) override;

	public:
	
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	float TraceRadius = 10;
	
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	float TraceStartOffset = -100;
	
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	float TraceEndOffset = 100;

	// vertical offset added to the raycast hit result height
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	float GroundHeightOffset = 0;
	
	UPROPERTY(EditAnywhere, Category = "Trace Settings")
	TEnumAsByte<ETraceTypeQuery> TraceChannel = TraceTypeQuery1;

#if WITH_EDITORONLY_DATA
	// Variable to output playback rate modifier to
	UPROPERTY()
	FName PlaybackRateOutputVariable_DEPRECATED;
#endif

	// Variable to output playback rate modifier to
	UPROPERTY(EditAnywhere, DisplayName = "Playback Rate Output Variable", Category = PlaybackRate)
	FAnimNextVariableReference PlaybackRateOutputVariableReference;

	UPROPERTY(EditAnywhere, Category = PlaybackRate)
	float MinPlaybackRateModifier = 0.5;
	UPROPERTY(EditAnywhere, Category = PlaybackRate)
	float MaxPlaybackRateModifier = 1.0;

	// only allow Yaw in the starting root transform when computing the future root motion
	UPROPERTY(EditAnywhere, Category = TraceSettings)
	bool bForceStartTransformUpright;
};