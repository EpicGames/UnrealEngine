// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_EvaluationNotifies.h"
#include "CommonAnimTypes.h"
#include "AnimNotifyState_TwoBoneIK.generated.h"

UCLASS(BlueprintType, DisplayName="Two Bone IK")
class UNotifyState_TwoBoneIK : public UAnimNotifyState
{
	GENERATED_BODY()
	
	public:
	
	/** Name of bone to control. This is the main bone chain to modify from. **/
	UPROPERTY(EditAnywhere, Category=IK)
	FBoneReference IKBone;
	
	/** Name of bone to IK Relative to - IK will target a position that has the same offset from the EffectorLocation, as the IKBone has from this Bone in the source pose. **/
    UPROPERTY(EditAnywhere, Category=IK)
    FBoneReference RelativeToBone;

	/** Limits to use if stretching is allowed. This value determines when to start stretch. For example, 0.9 means once it reaches 90% of the whole length of the limb, it will start apply. */
	UPROPERTY(EditAnywhere, Category=IK, meta = (editcondition = "bAllowStretching", ClampMin = "0.0", UIMin = "0.0"))
	double StartStretchRatio = 1.0;

	/** Limits to use if stretching is allowed. This value determins what is the max stretch scale. For example, 1.5 means it will stretch until 150 % of the whole length of the limb.*/
	UPROPERTY(EditAnywhere, Category= IK, meta = (editcondition = "bAllowStretching", ClampMin = "0.0", UIMin = "0.0"))
	double MaxStretchScale = 1.2;
	
	/** Limits to use if stretching is allowed. This value determins what is the max stretch scale. For example, 1.5 means it will stretch until 150 % of the whole length of the limb.*/
	UPROPERTY(EditAnywhere, Category= IK)
	float BlendInTime = 0.1f;
	UPROPERTY(EditAnywhere, Category= IK)
	float BlendOutTime = 0.1f;

	/** Effector Location. Target Location to reach. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effector, meta = (PinShownByDefault))
	FVector EffectorLocation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Effector, meta = (PinShownByDefault))
	FName EffectorLocationTransformName;

	UPROPERTY(EditAnywhere, Category=Effector)
	FBoneSocketTarget EffectorTarget;

	/** Joint Target Location. Location used to orient Joint bone. **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=JointTarget, meta=(PinShownByDefault))
	FVector JointTargetLocation = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category = JointTarget)
	FBoneSocketTarget JointTarget;

	/** Specify which axis it's aligned. Used when removing twist */
	UPROPERTY(EditAnywhere, Category = IK, meta = (editcondition = "!bAllowTwist"))
	FAxis TwistAxis;

	/** Reference frame of Effector Location. */
	UPROPERTY(EditAnywhere, Category=Effector)
	TEnumAsByte<enum EBoneControlSpace> EffectorLocationSpace = BCS_ComponentSpace;

	/** Reference frame of Joint Target Location. */
	UPROPERTY(EditAnywhere, Category=JointTarget)
	TEnumAsByte<enum EBoneControlSpace> JointTargetLocationSpace = BCS_ComponentSpace;

	/** Should stretching be allowed, to be prevent over extension */
	UPROPERTY(EditAnywhere, Category=IK)
	uint8 bAllowStretching:1 = false;

	/** Set end bone to use End Effector rotation */
	UPROPERTY(EditAnywhere, Category=IK)
	uint8 bTakeRotationFromEffectorSpace : 1 = false;

	/** Keep local rotation of end bone */
	UPROPERTY(EditAnywhere, Category = IK)
	uint8 bMaintainEffectorRelRot : 1 = false;

	/** Whether or not to apply twist on the chain of joints. This clears the twist value along the TwistAxis */
	UPROPERTY(EditAnywhere, Category = IK)
	uint8 bAllowTwist : 1 = true;
};

USTRUCT()
struct FTwoBoneIKNotifyInstance : public FEvaluationNotifyInstance
{
	FTwoBoneIKNotifyInstance()
    	:  CachedUpperLimbIndex(INDEX_NONE)
    	, CachedLowerLimbIndex(INDEX_NONE)
    {
    }

	
	GENERATED_BODY()

	virtual void Start(const UAnimSequenceBase* AnimationAsset) override;
	virtual void Update(const UAnimSequenceBase* AnimationAsset, float CurrentTime, float DeltaTime, bool bIsMirrored, const UMirrorDataTable* MirrorDataTable,
		FTransform& RootBoneTransform, const TMap<FName, FTransform>& NamedTransforms, FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;

	FBoneReference IKBone;
	FBoneReference RelativeToBone;
	FBoneSocketTarget EffectorTarget;
	
	UPROPERTY(EditAnywhere, Category = JointTarget)
	FBoneSocketTarget JointTarget;
	
	// cached limb index for upper
	FCompactPoseBoneIndex CachedUpperLimbIndex;
	
	// cached limb index for lower
	FCompactPoseBoneIndex CachedLowerLimbIndex;
};